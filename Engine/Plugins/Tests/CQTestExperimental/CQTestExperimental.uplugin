{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Experimental Code Quality Unreal Test Plugin",
	"Description": "Simplified testing of experimental features for Unreal Engine",
	"Category": "Testing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "CQTestExperimentalTests",
			"Type": "DeveloperTool",
			"LoadingPhase": "Default"
		}
	]
}