// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTestObjects.h"

#include "Net/UnrealNetwork.h"

///////////////////////////////////////////////////////////////////////
// AIrisTestReplicatedActor

AIrisTestReplicatedActor::AIrisTestReplicatedActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ReplicatedInt(0)
{
	bReplicates = true;
	bAlwaysRelevant = true;
}

void AIrisTestReplicatedActor::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	DOREPLIFETIME(AIrisTestReplicatedActor, ReplicatedInt);
}