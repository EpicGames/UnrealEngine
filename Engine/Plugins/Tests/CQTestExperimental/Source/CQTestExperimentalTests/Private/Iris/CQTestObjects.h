// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/GameInstance.h"
#include "GameFramework/Actor.h"
#include "GameFramework/GameModeBase.h"

#include "CQTestObjects.generated.h"

/** Basic GameInstance with a test value. */
UCLASS()
class UIrisTestGameInstanceClass : public UGameInstance
{
	GENERATED_BODY()

public:
	int TestValue{ 42 };
};

/** Basic GameMode with a test value. */
UCLASS()
class AIrisTestGameMode : public AGameModeBase
{
	GENERATED_BODY()

public:
	int32 TestValue{ 42 };
};

/** Basic Actor used for replication tests. */
UCLASS(NotBlueprintable)
class AIrisTestReplicatedActor : public AActor
{
	GENERATED_BODY()

public:
	AIrisTestReplicatedActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

	UPROPERTY(Replicated)
	int32 ReplicatedInt;
};