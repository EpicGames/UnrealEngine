// Copyright Epic Games, Inc. All Rights Reserved.

#include "CQTest.h"
#include "Components/PIENetworkComponent.h"
#include "CQTestObjects.h"
#include "Tests/AutomationCommon.h"

#include "Engine/World.h"

#if ENABLE_PIE_NETWORK_TEST && UE_WITH_IRIS

/**
 * The tests below showcase how the `FBasePIENetworkComponent` can be used to setup and simulate a client/server network using the experimental iris replication system (https://dev.epicgames.com/documentation/en-us/unreal-engine/iris-replication-system-in-unreal-engine).
 * See the CQTest Engine plugin for more information on tests using the default replication system.
*/

NETWORK_TEST_CLASS(IrisStateTest, "TestFramework.CQTest.Experimental.Network")
{
	struct DerivedState : public FBasePIENetworkComponentState
	{
		int32 IndependentNumber = 0;
	};

	int32 SharedNumber = 0;
	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };
	
	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("1"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithClients(3)
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(AGameModeBase::StaticClass())
			.Build(Network);
	}
	
	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(Network_WithMultipleSteps_TriggersStepsInOrder)
	{
		Network
			.ThenServer([this](DerivedState&) { ASSERT_THAT(AreEqual(0, SharedNumber++)); })
			.ThenClient(0, [this](DerivedState&) { ASSERT_THAT(AreEqual(1, SharedNumber++)); })
			.ThenServer([this](DerivedState&) { ASSERT_THAT(AreEqual(2, SharedNumber++)); })
			.Then([this]() { ASSERT_THAT(AreEqual(3, SharedNumber++)); });
	}

	TEST_METHOD(Network_WithServerCommands_RetainsStateBetweenCalls)
	{
		Network
			.ThenServer([this](DerivedState& State) { ASSERT_THAT(AreEqual(0, State.IndependentNumber++)); })
			.ThenServer([this](DerivedState& State) { ASSERT_THAT(AreEqual(1, State.IndependentNumber++)); });
	}

	TEST_METHOD(Network_WithClientCommands_RetainsStateBetweenCalls)
	{
		Network
			.ThenClient(0, [this](DerivedState& State) { ASSERT_THAT(AreEqual(0, State.IndependentNumber++)); })
			.ThenClient(0, [this](DerivedState& State) { ASSERT_THAT(AreEqual(1, State.IndependentNumber++)); });
	}

	TEST_METHOD(Network_WithClientAndServerCommands_DoNotShareState)
	{
		Network
			.ThenServer([this](DerivedState& State) { State.IndependentNumber++; })
			.ThenClient(0, [this](DerivedState& State) { ASSERT_THAT(AreEqual(0, State.IndependentNumber)); });
	}

	TEST_METHOD(Network_WithMultipleClients_DoNotShareState)
	{
		Network
			.ThenClients([this](DerivedState& State) { State.IndependentNumber = State.ClientIndex; })
			.ThenClients([this](DerivedState& State) { ASSERT_THAT(AreEqual(State.ClientIndex, State.IndependentNumber)); });
	}

	TEST_METHOD(Network_WithTickingServerCommand_TicksUntilDone)
	{
		Network
			.UntilServer([this](DerivedState& State) { return ++State.IndependentNumber > 4; })
			.ThenServer([this](DerivedState& State) { ASSERT_THAT(AreEqual(State.IndependentNumber, 5)); });
	}

	TEST_METHOD(Network_WithTickingClientCommands_TicksEachCommand)
	{
		Network
			.UntilClients([this](DerivedState& State) { SharedNumber++;  return ++State.IndependentNumber > 4; })
			.Then([this]() { ASSERT_THAT(AreEqual(15, SharedNumber)); });
	}

	TEST_METHOD(Network_WithClients_ClientConnectionsAreFilledIn)
	{
		Network
			.ThenServer([this](DerivedState& State) {
				ASSERT_THAT(AreEqual(State.ClientCount, 3));
				ASSERT_THAT(AreEqual(State.ClientConnections.Num(), 3));
				for (int32 ClientIndex = 0; ClientIndex < State.ClientConnections.Num(); ++ ClientIndex)
				{
					ASSERT_THAT(IsNotNull(State.ClientConnections[ClientIndex]));
				}
			});
	}
};

NETWORK_TEST_CLASS(IrisReplicationTest, "TestFramework.CQTest.Experimental.Network")
{
	struct DerivedState : public FBasePIENetworkComponentState
	{
		AIrisTestReplicatedActor* ReplicatedActor = nullptr;
	};

	const int32 ExpectedReplicatedValue = 42;
	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };
	
	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("1"));
	}

	BEFORE_EACH() 
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(AGameModeBase::StaticClass())
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(SpawnAndReplicateActor_WithReplicatedActor_ProvidesActorToClients)
	{
		Network.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor>()
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(IsNotNull(ServerState.ReplicatedActor));
			})
			.ThenClients([this](DerivedState& ClientState) {
				ASSERT_THAT(IsNotNull(ClientState.ReplicatedActor));
			});
	}

	TEST_METHOD(SpawnAndReplicateActor_ThenUpdateProperty_UpdatesPropertyOnClients)
	{
		Network.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor>()
			.ThenServer(TEXT("Server Set Value"), [this](DerivedState& ServerState) {
				ServerState.ReplicatedActor->ReplicatedInt = ExpectedReplicatedValue;
			})
			.UntilClients(TEXT("Clients Check Value"), [this](DerivedState& ClientState) {
				return ClientState.ReplicatedActor->ReplicatedInt == ExpectedReplicatedValue;
			});
	}

	TEST_METHOD(SpawnAndReplicateActor_WithSpawnParameters_PassesParametersToSpawnedObject)
	{
		FActorSpawnParameters SpawnParameters;
		SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		Network.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor>(SpawnParameters)
			.UntilClients([SpawnParameters](DerivedState& ClientState) {
				return SpawnParameters.SpawnCollisionHandlingOverride == ClientState.ReplicatedActor->SpawnCollisionHandlingMethod;
			});
	}

	TEST_METHOD(SpawnAndReplicateActor_WithBeforeReplicates_InvokesBeforeReplicate) {
		auto BeforeReplicate = [this](AIrisTestReplicatedActor& Actor) {
			Actor.ReplicatedInt = ExpectedReplicatedValue;
		};

		Network.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor>(BeforeReplicate)
			.UntilClients([this](DerivedState& ClientState) {
				return ClientState.ReplicatedActor->ReplicatedInt == ExpectedReplicatedValue;
			});
	}

	TEST_METHOD(SpawnAndReplicateActor_WithSpawnParametersAndBeforeReplicate_UsesBoth)
	{
		auto BeforeReplicate = [this](AIrisTestReplicatedActor& Actor) {
			Actor.ReplicatedInt = ExpectedReplicatedValue;
		};

		FActorSpawnParameters SpawnParameters;
		SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		Network.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor>(SpawnParameters, BeforeReplicate)
			.UntilClients([this, SpawnParameters](DerivedState& ClientState) {
			AIrisTestReplicatedActor& actor = *ClientState.ReplicatedActor;
				return SpawnParameters.SpawnCollisionHandlingOverride == actor.SpawnCollisionHandlingMethod && actor.ReplicatedInt == ExpectedReplicatedValue;
			});
	}
};

NETWORK_TEST_CLASS(IrisMultipleActorStateReplication, "TestFramework.CQTest.Experimental.Network")
{
	struct DerivedState : public FBasePIENetworkComponentState
	{
		AIrisTestReplicatedActor* ReplicatedActor1 = nullptr;
		AIrisTestReplicatedActor* ReplicatedActor2 = nullptr;
	};

	const int32 Actor1ExpectedReplicatedValue = 42;
	const int32 Actor2ExpectedReplicatedValue = 24;

	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("1"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithClients(1)
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	auto MakeSetInt(int32 Value) 
	{
		return [Value](AIrisTestReplicatedActor& Actor) {
			Actor.ReplicatedInt = Value;
		};
	}

	TEST_METHOD(SpawnAndReplicateActor_WithMultipleActors_ReplicatesBoth)
	{
		Network.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor1>(MakeSetInt(Actor1ExpectedReplicatedValue))
			.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor2>(MakeSetInt(Actor2ExpectedReplicatedValue))
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(AreEqual(Actor1ExpectedReplicatedValue, ServerState.ReplicatedActor1->ReplicatedInt));
				ASSERT_THAT(AreEqual(Actor2ExpectedReplicatedValue, ServerState.ReplicatedActor2->ReplicatedInt));
			})
			.ThenClients([this](DerivedState& ClientState) {
				ASSERT_THAT(AreEqual(Actor1ExpectedReplicatedValue, ClientState.ReplicatedActor1->ReplicatedInt));
				ASSERT_THAT(AreEqual(Actor2ExpectedReplicatedValue, ClientState.ReplicatedActor2->ReplicatedInt));
			});
	}

	TEST_METHOD(SpawnAndReplicate_WithLateJoin_ReplicatesBoth)
	{
		Network.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor1>(MakeSetInt(42))
			.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor2>(MakeSetInt(24))
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(AreEqual(Actor1ExpectedReplicatedValue, ServerState.ReplicatedActor1->ReplicatedInt));
				ASSERT_THAT(AreEqual(Actor2ExpectedReplicatedValue, ServerState.ReplicatedActor2->ReplicatedInt));
			})
			.ThenClientJoins()
			.ThenClients([this](DerivedState& ClientState) {
				ASSERT_THAT(AreEqual(Actor1ExpectedReplicatedValue, ClientState.ReplicatedActor1->ReplicatedInt));
				ASSERT_THAT(AreEqual(Actor2ExpectedReplicatedValue, ClientState.ReplicatedActor2->ReplicatedInt));
			});
	}
};

NETWORK_TEST_CLASS(IrisLateJoinTest, "TestFramework.CQTest.Experimental.Network")
{
	struct DerivedState : public FBasePIENetworkComponentState
	{
		AIrisTestReplicatedActor* ReplicatedActor = nullptr;
	};

	FPIENetworkComponent<DerivedState> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("1"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder<DerivedState>()
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(AGameModeBase::StaticClass())
			.WithClients(1)
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(ThenClientJoins_AfterStart_AddsClient) 
	{
		Network.ThenServer([this](DerivedState& ServerState) {
			ASSERT_THAT(AreEqual(ServerState.ClientCount, 1));
		})
		.ThenClientJoins()
		.ThenServer([this](DerivedState& ServerState) {
			ASSERT_THAT(AreEqual(ServerState.ClientCount, 2));
			ASSERT_THAT(AreEqual(ServerState.ClientConnections.Num(), 2));
			for (int32 ClientIndex = 0; ClientIndex < ServerState.ClientConnections.Num(); ++ClientIndex)
			{
				ASSERT_THAT(IsNotNull(ServerState.ClientConnections[ClientIndex]));
			}
		});
	}

	TEST_METHOD(ThenClientJoins_AfterStart_ReplicatesState) 
	{
		Network.SpawnAndReplicate<AIrisTestReplicatedActor, &DerivedState::ReplicatedActor>()
			.ThenServer([this](DerivedState& ServerState) {
				ASSERT_THAT(IsNotNull(ServerState.ReplicatedActor));
			})
			.ThenClient(0, [this](DerivedState& ClientState) {
				ASSERT_THAT(IsNotNull(ClientState.ReplicatedActor));
			})
			.ThenClientJoins()
			.ThenClient(1, [this](DerivedState& ClientState) {
				ASSERT_THAT(IsNotNull(ClientState.ReplicatedActor));
			});
	}
};

NETWORK_TEST_CLASS(IrisGameInstanceTest, "TestFramework.CQTest.Experimental.Network")
{
	const int32 ExpectedReplicatedValue = 42;

	FPIENetworkComponent<FBasePIENetworkComponentState> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("1"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder()
			.WithGameInstanceClass(FSoftClassPath(UIrisTestGameInstanceClass::StaticClass()))
			.WithGameMode(AGameModeBase::StaticClass())
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(NetworkComponent_WithGameInstanceClass_BuildsNetworkWithProvidedGameInstance)
	{
		Network.ThenServer([this](FBasePIENetworkComponentState& ServerState) {
			UIrisTestGameInstanceClass* GameInstance = Cast<UIrisTestGameInstanceClass>(ServerState.World->GetGameInstance());
			ASSERT_THAT(IsNotNull(GameInstance));
			ASSERT_THAT(AreEqual(ExpectedReplicatedValue, GameInstance->TestValue));
		});
	}
};

NETWORK_TEST_CLASS(IrisGameModeTest, "TestFramework.CQTest.Experimental.Network")
{
	const int32 ExpectedReplicatedValue = 42;

	FPIENetworkComponent<> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("1"));
	}

	BEFORE_EACH()
	{
		FNetworkComponentBuilder()
			.WithGameInstanceClass(UGameInstance::StaticClass())
			.WithGameMode(AIrisTestGameMode::StaticClass())
			.Build(Network);
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(NetworkComponent_WithGameMode_BuildsNetworkWithProvidedGameMode)
	{
		Network.ThenServer([this](FBasePIENetworkComponentState& ServerState) {
			AIrisTestGameMode* GameMode = Cast<AIrisTestGameMode>(ServerState.World->GetAuthGameMode());
			ASSERT_THAT(IsNotNull(GameMode));
			ASSERT_THAT(AreEqual(ExpectedReplicatedValue, GameMode->TestValue));
		});
	}
};

NETWORK_TEST_CLASS(IrisSetupErrorTest, "TestFramework.CQTest.Experimental.Network")
{
	FPIENetworkComponent<> Network{ TestRunner, TestCommandBuilder, bInitializing };

	inline static TSharedPtr<FScopedTestEnvironment> TestEnvironment{ nullptr };
	BEFORE_ALL()
	{
		TestEnvironment = FScopedTestEnvironment::Get();
		TestEnvironment->SetConsoleVariableValue(TEXT("net.Iris.UseIrisReplication"), TEXT("1"));
	}

	AFTER_EACH() 
	{
		const FString ExpectedError = TEXT("Failed to initialize Network Component");
		
		FAutomationTestExecutionInfo TestInfo;
		TestRunner->GetExecutionInfo(TestInfo);

		TestInfo.RemoveAllEvents([&ExpectedError](FAutomationEvent& event) {
			return event.Message.Contains(ExpectedError);
		});

		if (TestInfo.GetErrorTotal() == 0)
		{
			TestRunner->ClearExecutionInfo();
		}
	}

	AFTER_ALL()
	{
		TestEnvironment = nullptr;
	}

	TEST_METHOD(NetworkComponent_WithoutUsingBuilder_AddsErrorAndDoesNotCrash)
	{
		Network.ThenServer([this](FBasePIENetworkComponentState&) { Assert.Fail("Unexpected Error"); });
	}
};

#endif // ENABLE_PIE_NETWORK_TEST && UE_WITH_IRIS